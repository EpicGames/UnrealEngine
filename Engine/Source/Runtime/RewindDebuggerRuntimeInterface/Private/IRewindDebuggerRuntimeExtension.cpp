// Copyright Epic Games, Inc. All Rights Reserved.

#include "RewindDebuggerRuntimeInterface/IRewindDebuggerRuntimeExtension.h"
#include "Features/IModularFeatures.h"

const FName IRewindDebuggerRuntimeExtension::ModularFeatureName = "RewindDebuggerRuntimeExtension";

namespace RewindDebugger
{

void IterateExtensions(TFunctionRef<void(IRewindDebuggerRuntimeExtension* Extension)> IteratorFunction)
{
	// update extensions
	IModularFeatures& ModularFeatures = IModularFeatures::Get();

	const int32 NumExtensions = ModularFeatures.GetModularFeatureImplementationCount(IRewindDebuggerRuntimeExtension::ModularFeatureName);
	for (int32 ExtensionIndex = 0; ExtensionIndex < NumExtensions; ++ExtensionIndex)
	{
		IRewindDebuggerRuntimeExtension* Extension = static_cast<IRewindDebuggerRuntimeExtension*>(ModularFeatures.GetModularFeatureImplementation(IRewindDebuggerRuntimeExtension::ModularFeatureName, ExtensionIndex));
		IteratorFunction(Extension);
	}
}

} // RewindDebugger

