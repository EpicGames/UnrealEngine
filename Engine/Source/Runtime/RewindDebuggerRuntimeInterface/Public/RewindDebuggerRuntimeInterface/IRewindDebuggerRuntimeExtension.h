// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "CoreTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8

#include "UObject/NameTypes.h"
#include "Features/IModularFeature.h"
#include "Templates/FunctionFwd.h"
#include "Templates/SharedPointerFwd.h"

#define UE_API REWINDDEBUGGERRUNTIMEINTERFACE_API

class FMessageEndpoint;
class IRewindDebuggerRuntimeExtension;

namespace UE::TraceBasedDebuggers
{
struct FRemoteSessionsManager;
}

namespace RewindDebugger
{
UE_API extern void IterateExtensions(TFunctionRef<void(IRewindDebuggerRuntimeExtension* Extension)> IteratorFunction);
}

struct FMessageEndpointBuilder;

/**
 * IRewindDebuggerRuntimeExtension
 *
 * Interface class for extensions which add functionality to the rewind debugger runtime.
 * These get callbacks on recording start/stop, to enable/disable trace channels and on clear to clean up any cached data.
 * They can also register and handle messages for remote sessions debugging.
 */
class IRewindDebuggerRuntimeExtension : public IModularFeature
{
public:
	virtual ~IRewindDebuggerRuntimeExtension() = default;

	static UE_API const FName ModularFeatureName;

	/**
	 * This method is called once the trace connection is established (i.e., FTraceAuxiliary::OnConnection delegate) which can be
	 * from any thread.
	 * This is the method to override to enable trace channels specific to the extension.
	 */
	virtual void RecordingStarted()
	{
	}

	/** This method is called from the game thread when recording gets stopped */
	virtual void RecordingStopped()
	{
	}

	/** This method is called from the game thread before executing the "start recording" command (i.e., FTraceAuxiliary::Start/Relay methods) */
	virtual void Clear()
	{
	}

	/**
	 * This method allows extensions to register their own message type handlers to the message system and send response using
	 * the remote sessions manager.
	 * @param InRemoteSessionsManager The remote sessions manager that can be used to send response messages
	 * @param InMessageEndpointBuilder The message endpoint builder where message type handlers needs to be registered.
	 */
	virtual void RegisterMessageHandlers(const TSharedPtr<UE::TraceBasedDebuggers::FRemoteSessionsManager>& InRemoteSessionsManager, FMessageEndpointBuilder& InMessageEndpointBuilder)
	{
	}

	/**
	 * This method allows extensions to register their own message types in the remote sessions manager and subscribe in the message endpoint.
	 * @param InRemoteSessionsManager The remote sessions manager that can be used to register supported messages
	 * @param InMessageEndpoint The message endpoint to subscribe to specific message types
	 */
	virtual void RegisterMessageTypes(const TSharedPtr<UE::TraceBasedDebuggers::FRemoteSessionsManager>& InRemoteSessionsManager, FMessageEndpoint& InMessageEndpoint)
	{
	}
};

#undef UE_API
